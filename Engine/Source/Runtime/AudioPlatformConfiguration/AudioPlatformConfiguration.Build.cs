// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AudioPlatformConfiguration : ModuleRules
	{
		public AudioPlatformConfiguration(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject"
				}
			);

			PublicIncludePathModuleNames.AddRange(
				new string[]
				{
					"CoreUObject"
				}
			);

			PrivateIncludePathModuleNames.Add("Engine");

			AddEngineThirdPartyPrivateStaticDependencies(Target, "UELibSampleRate");
			PrivateDefinitions.Add("WITH_LIBSAMPLERATE=1");
		}
	}
}
