// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioCompressionSettings.h"
#include "AudioCompressionSettingsUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioCompressionSettings)

FPlatformRuntimeAudioCompressionOverrides::FPlatformRuntimeAudioCompressionOverrides()
	: bOverrideCompressionTimes(false)
	, DurationThreshold(5.0f)
	, MaxNumRandomBranches(0)
	, SoundCueQualityIndex(0)
{
	
}

FPlatformRuntimeAudioCompressionOverrides* FPlatformRuntimeAudioCompressionOverrides::DefaultCompressionOverrides = nullptr;

// Increment this return value to force a recook on all Stream Caching assets.
// For testing, it's useful to set this to either a negative number or
// absurdly large number, to ensure you do not pollute the DDC.
int32 FPlatformAudioCookOverrides::GetStreamCachingVersion()
{
	return 5028;
}


void FPlatformAudioCookOverrides::GetHashSuffix(const FPlatformAudioCookOverrides* InOverrides, FString& OutSuffix)
{
	if (InOverrides == nullptr)
	{
		return;
	}	
	
	using FPCU = FPlatformCompressionUtilities;
	FString SoundWaveHash;

	// Starting Delim is important, as FSoundWaveData::FindRuntimeFormat, uses it determine format from the inline chunk name.
	OutSuffix += TEXT("_");

	// Start with StreamCache version.
	FPCU::AppendHash(OutSuffix, TEXT("SCVER"), GetStreamCachingVersion());
	
	// Each member in declaration order.
	
	// FPlatformAudioCookOverrides
	FPCU::AppendHash(OutSuffix, TEXT("R4DV"), InOverrides->bResampleForDevice);

	TArray<float> Rates;
	InOverrides->PlatformSampleRates.GenerateValueArray(Rates);
	for (int32 i = 0; i < Rates.Num(); ++i)
	{
		FPCU::AppendHash(OutSuffix, *FString::Printf(TEXT("SR%d"), i), Rates[i]);
	}

	FPCU::AppendHash(OutSuffix, TEXT("QMOD"), InOverrides->CompressionQualityModifier);
	FPCU::AppendHash(OutSuffix, TEXT("CQLT"), InOverrides->SoundCueCookQualityIndex);
	FPCU::AppendHash(OutSuffix, TEXT("ASTH"), InOverrides->AutoStreamingThreshold);
	FPCU::AppendHash(OutSuffix, TEXT("INLC"), InOverrides->bInlineFirstAudioChunk);
	FPCU::AppendHash(OutSuffix, TEXT("LCK1"), InOverrides->LengthOfFirstAudioChunkInSecs);
	
	// FAudioStreamCachingSettings
	FPCU::AppendHash(OutSuffix, TEXT("CSZE"), InOverrides->StreamCachingSettings.CacheSizeKB);
	FPCU::AppendHash(OutSuffix, TEXT("LCF"), InOverrides->StreamCachingSettings.bForceLegacyStreamChunking);
	FPCU::AppendHash(OutSuffix, TEXT("ZCS"), InOverrides->StreamCachingSettings.ZerothChunkSizeForLegacyStreamChunkingKB);
	FPCU::AppendHash(OutSuffix, TEXT("MCSO"), InOverrides->StreamCachingSettings.MaxChunkSizeOverrideKB);
	
	OutSuffix += TEXT("END");
}
