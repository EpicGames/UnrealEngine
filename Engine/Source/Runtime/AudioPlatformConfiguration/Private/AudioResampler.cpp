// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioResampler.h"

// Convenience macro for the case in which LibSampleRate needs to be built for limited platforms.
#ifndef WITH_LIBSAMPLERATE
#define WITH_LIBSAMPLERATE (WITH_EDITOR && !PLATFORM_LINUX)
#endif

DEFINE_LOG_CATEGORY(LogAudioResampler);

#if WITH_LIBSAMPLERATE
#include "samplerate.h"
#endif // WITH_LIBSAMPLERATE

namespace Audio
{
	// Helper function to ensure that buffers are appropriately set up.
	bool CheckBufferValidity(const FResamplingParameters& InParameters, FResamplerResults& OutData)
	{
		if (OutData.OutBuffer == nullptr)
		{
			UE_LOGF(LogAudioResampler, Error, "Please specify an output buffer when using Resample().");
			return false;
		}

		if (InParameters.SourceSampleRate <= 0.0f || InParameters.DestinationSampleRate <= 0.0f)
		{
			UE_LOGF(LogAudioResampler, Error, "Please use non-zero, positive sample rates when calling Resample().");
			return false;
		}

		if (OutData.OutBuffer->Num() < GetOutputBufferSize(InParameters))
		{
			UE_LOGF(LogAudioResampler, Error, "Insufficient space in output buffer: Please allocate space for %d samples.", GetOutputBufferSize(InParameters));
			return false;
		}

		return true;
	}

	int32 GetOutputBufferSize(const FResamplingParameters& InParameters)
	{
		const int32 NumChannels = FMath::Max(1, InParameters.NumChannels);
		const int32 NumInputFrames = InParameters.InputBuffer.Num() / NumChannels;
		const float Ratio = InParameters.DestinationSampleRate / InParameters.SourceSampleRate;
		const int32 NumOutputFrames = FMath::CeilToInt(Ratio * NumInputFrames);

		return NumChannels * NumOutputFrames;
	}

	bool Resample(const FResamplingParameters& InParameters, FResamplerResults& OutData)
	{
#if WITH_LIBSAMPLERATE
		// Check validity of buffers.
		if (!CheckBufferValidity(InParameters, OutData))
		{
			return false;
		}

		// Create new converter
		int32 Error = 0;
#ifdef LIBSAMPLERATE_WITHOUT_SINC
		SRC_STATE* Converter = src_new(SRC_LINEAR, InParameters.NumChannels, &Error);
#else
		SRC_STATE* Converter = src_new(SRC_SINC_BEST_QUALITY, InParameters.NumChannels, &Error);
#endif
		if (Converter == nullptr || Error != 0)
		{
			UE_LOGF(LogTemp, Error, "Error creating sample converter: %s", src_strerror(Error));
			return false;
		}

		SRC_DATA SrcData;
		SrcData.data_in = InParameters.InputBuffer.GetData();
		SrcData.data_out = OutData.OutBuffer->GetData();
		SrcData.input_frames = InParameters.InputBuffer.Num() / InParameters.NumChannels;
		SrcData.output_frames = OutData.OutBuffer->Num() / InParameters.NumChannels;
		SrcData.src_ratio = InParameters.DestinationSampleRate / InParameters.SourceSampleRate;
		SrcData.end_of_input = 1;

		Error = src_process(Converter, &SrcData);

		if (Error != 0)
		{
			UE_LOGF(LogTemp, Error, "Error on Resampling process: %s", src_strerror(Error));
			return false;
		}

		OutData.InputFramesUsed = SrcData.input_frames_used;
		OutData.OutputFramesGenerated = SrcData.output_frames_gen;

		// Clean up:
		src_delete(Converter);
#endif //WITH_LIBSAMPLERATE
		return true;
	}

	class FResamplerImpl
	{
	public:
		FResamplerImpl();
		~FResamplerImpl();

		void Init(EResamplingMethod ResamplingMethod, float StartingSampleRateRatio, int32 InNumChannels);
		void SetSampleRateRatio(float InRatio);
		int32 ProcessAudio(float* InAudioBuffer, int32 InSamples, bool bEndOfInput, float* OutAudioBuffer, int32 MaxOutputFrames, int32& OutNumFrames);

#if WITH_LIBSAMPLERATE
		float CurrentSampleRateRatio;
		SRC_STATE* LibSRCState;
		SRC_DATA Data;
#endif

	};

#if WITH_LIBSAMPLERATE
	FResamplerImpl::FResamplerImpl() 
		: CurrentSampleRateRatio(-1.0f)
		, LibSRCState(nullptr)
	{
	}
	
	FResamplerImpl::~FResamplerImpl() 
	{
		if (LibSRCState)
		{
			LibSRCState = src_delete(LibSRCState);
		}
		check(!LibSRCState);
	}
	
	void FResamplerImpl::Init(EResamplingMethod ResamplingMethod, float StartingSampleRateRatio, int32 InNumChannels) 
	{
		int32 ErrorResult = 0;

		// Reset the SRC state if we already have one with equal channels
		if (LibSRCState)
		{
			if (InNumChannels == src_get_channels(LibSRCState))
			{
				ErrorResult = src_reset(LibSRCState);
				if (ErrorResult != 0)
				{
					const char* ErrorString = src_strerror(ErrorResult);
					UE_LOGF(LogAudioResampler, Error, "Failed to reset sample converter state: %s.", ErrorString);
					return;
				}
			}
			else
			{
				// If the channel counts do not match, then remove one.
				LibSRCState = src_delete(LibSRCState);
				check(nullptr == LibSRCState);
			}
		}

		// Create a new one if one does not exist.
		if (nullptr == LibSRCState)	
		{
			LibSRCState = src_new((int32)ResamplingMethod, InNumChannels, &ErrorResult);
			if (!LibSRCState)
			{
				const char* ErrorString = src_strerror(ErrorResult);
				UE_LOGF(LogAudioResampler, Error, "Failed to create a sample rate convertor state object: %s.", ErrorString);
			}
		}

		if (LibSRCState)
		{
			ErrorResult = src_set_ratio(LibSRCState, StartingSampleRateRatio);
			if (ErrorResult != 0)
			{
				const char* ErrorString = src_strerror(ErrorResult);
				UE_LOGF(LogAudioResampler, Error, "Failed to set sample rate ratio: %s.", ErrorString);
			}
		}

		CurrentSampleRateRatio = StartingSampleRateRatio;
	}
	
	void FResamplerImpl::SetSampleRateRatio(float InRatio) 
	{
		CurrentSampleRateRatio = FMath::Max(InRatio, 0.00001f);
	}
	
	int32 FResamplerImpl::ProcessAudio(float* InAudioBuffer, int32 InSamples, bool bEndOfInput, float* OutAudioBuffer, int32 MaxOutputFrames, int32& OutNumFrames)
	{
		if (LibSRCState)
		{
			Data.data_in = InAudioBuffer;
			Data.input_frames = InSamples;
			Data.data_out = OutAudioBuffer;
			Data.output_frames = MaxOutputFrames;
			Data.src_ratio = (double)CurrentSampleRateRatio;
			Data.end_of_input = bEndOfInput ? 1 : 0;

			int32 ErrorResult = src_process(LibSRCState, &Data);
			if (ErrorResult != 0)
			{
				const char* ErrorString = src_strerror(ErrorResult);
				UE_LOGF(LogAudioResampler, Error, "Failed to process audio: %s.", ErrorString);
				return ErrorResult;
			}

			OutNumFrames = Data.output_frames_gen;
		}
		return 0; 
	}

#else
	// Null implementation
	FResamplerImpl::FResamplerImpl() {}
	FResamplerImpl::~FResamplerImpl() {}
	void FResamplerImpl::Init(EResamplingMethod ResamplingMethod, float StartingSampleRateRatio, int32 InNumChannels) {}
	void FResamplerImpl::SetSampleRateRatio(float InRatio) {}	
	int32 FResamplerImpl::ProcessAudio(float* InAudioBuffer, int32 InSamples, bool bEndOfInput, float* OutAudioBuffer, int32 MaxOutputFrames, int32& OutNumFrames) { return 0; }
#endif

	FResampler::FResampler()
	{
		Impl = CreateImpl();
	}
	
	FResampler::~FResampler()
	{

	}

	void FResampler::Init(EResamplingMethod ResamplingMethod, float StartingSampleRateRatio, int32 InNumChannels)
	{
		if (Impl.IsValid())
		{
			Impl->Init(ResamplingMethod, StartingSampleRateRatio, InNumChannels);
		}
	}

	void FResampler::SetSampleRateRatio(float InRatio)
	{
		if (Impl.IsValid())
		{
			Impl->SetSampleRateRatio(InRatio);
		}
	}

	int32 FResampler::ProcessAudio(float* InAudioBuffer, int32 InSamples, bool bEndOfInput, float* OutAudioBuffer, int32 MaxOutputFrames, int32& OutNumFrames)
	{
		if (Impl.IsValid())
		{
			return Impl->ProcessAudio(InAudioBuffer, InSamples, bEndOfInput, OutAudioBuffer, MaxOutputFrames, OutNumFrames);
		}
		return 0;
	}

	TUniquePtr<FResamplerImpl> FResampler::CreateImpl()
	{
		return TUniquePtr<FResamplerImpl>(new FResamplerImpl());
	}
}

