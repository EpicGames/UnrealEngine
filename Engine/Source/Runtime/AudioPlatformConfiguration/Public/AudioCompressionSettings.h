// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Math/UnrealMathSSE.h"
#include "Templates/Tuple.h"
#include "UObject/ObjectMacros.h"

#include "AudioCompressionSettings.generated.h"

UENUM()
enum class ESoundwaveSampleRateSettings : uint8
{
	Max,
	High,
	Medium,
	Low,
	Min
};

/************************************************************************/
/* FAudioStreamCachingSettings                                           */
/* Properties used to determine chunk sizes for the two caches used     */
/* when the experimental Stream Caching feature is used.                */
/************************************************************************/
struct FAudioStreamCachingSettings
{
	static constexpr int32 DefaultCacheSize = 64 * 1024;

	// Target memory usage, in kilobytes.
	// In the future settings for the cache can be more complex, but for now
	// we set the max chunk size to 256 kilobytes, then set the number of elements in our cache as
	// CacheSizeKB / 256.
	int32 CacheSizeKB;

	// Bool flag for keeping sounds flagged for streaming chunked in the style of the legacy streaming manager.
	bool bForceLegacyStreamChunking;

	int32 ZerothChunkSizeForLegacyStreamChunkingKB;

	// will be ignored if < 0
	int32 MaxChunkSizeOverrideKB;

	FAudioStreamCachingSettings()
		: CacheSizeKB(DefaultCacheSize)
		, bForceLegacyStreamChunking(false)
		, ZerothChunkSizeForLegacyStreamChunkingKB(256)
		, MaxChunkSizeOverrideKB(INDEX_NONE)
	{
	}
};

/************************************************************************/
/* FPlatformAudioCookOverrides                                          */
/* This struct is used for settings used during the cook to a target    */
/* platform (platform-specific compression quality and resampling, etc.)*/
/************************************************************************/
struct FPlatformAudioCookOverrides
{
	// Increment this return value to force a recook on all Stream Caching assets.
	// For testing, it's useful to set this to either a negative number or
	// absurdly large number, to ensure you do not pollute the DDC.
	static AUDIOPLATFORMCONFIGURATION_API int32 GetStreamCachingVersion();

	bool bResampleForDevice;

	// Mapping of which sample rates are used for each sample rate quality for a specific platform.
	TMap<ESoundwaveSampleRateSettings, float> PlatformSampleRates;

	// Scales all compression qualities when cooking to this platform. For example, 0.5 will halve all compression qualities, and 1.0 will leave them unchanged.
	float CompressionQualityModifier;

	// If set, the cooker will keep only this level of quality
	int32 SoundCueCookQualityIndex = INDEX_NONE;

	// When set to any platform > 0.0, this will automatically set any USoundWave beyond this value to be streamed from disk.
	// If StreamCaching is set to true, this will be used 
	float AutoStreamingThreshold;

	// Wether to inline the first "Audio" chunk, which is typically chunk 1. (Only on assets marked retain-on-load with a size of audio in secs set) 
	bool bInlineFirstAudioChunk = false;

	// This will decide how much data to put in the first audio chunk. Anything <= 0 will be ignored.
	// Must be combined with bInlineFirstAudioChunk, this will decide how much data to put in the first chunk.
	// NOTE: This is platform default and can be overriden by each asset or soundclass.
	float LengthOfFirstAudioChunkInSecs = 0.f;

	// If Load On Demand is enabled, these settings are used to determine chunks and cache sizes.
	FAudioStreamCachingSettings StreamCachingSettings;

	FPlatformAudioCookOverrides()
		: bResampleForDevice(false)
		, CompressionQualityModifier(1.0f)
		, AutoStreamingThreshold(0.0f)
		, bInlineFirstAudioChunk(false)
		, LengthOfFirstAudioChunkInSecs(0.f)
	{
		PlatformSampleRates.Add(ESoundwaveSampleRateSettings::Max, 48000);
		PlatformSampleRates.Add(ESoundwaveSampleRateSettings::High, 32000);
		PlatformSampleRates.Add(ESoundwaveSampleRateSettings::Medium, 24000);
		PlatformSampleRates.Add(ESoundwaveSampleRateSettings::Low, 12000);
		PlatformSampleRates.Add(ESoundwaveSampleRateSettings::Min, 8000);
	}

	// This is used to invalidate compressed audio for a specific platform.
	static AUDIOPLATFORMCONFIGURATION_API void GetHashSuffix(const FPlatformAudioCookOverrides* InOverrides, FString& OutSuffix);
};

USTRUCT()
struct FPlatformRuntimeAudioCompressionOverrides
{
	GENERATED_USTRUCT_BODY()

	// When true, overrides the Sound Group on each Sound Wave, and instead uses the Duration Threshold value to determine whether a sound should be fully decompressed during initial loading.
	UPROPERTY(EditAnywhere, Category = "DecompressOnLoad")
	bool bOverrideCompressionTimes;
	
	// When Override Compression Times is set to true, any sound under this threshold (in seconds) will be fully decompressed on load.
	// Otherwise the first chunk of this sound is cached at load and the rest is decompressed in real time.
	// If set to zero, will default to the Sound Group on the relevant Sound Wave
	UPROPERTY(EditAnywhere, Category = "DecompressOnLoad")
	float DurationThreshold;

	// On this platform, any random nodes on Sound Cues will automatically only preload this number of branches and dispose of any others
	// on load. This can drastically cut down on memory usage. If set to 0, no branches are culled.
	UPROPERTY(EditAnywhere, Category = "SoundCueLoading", meta = (DisplayName = "Maximum Branches on Random SoundCue nodes", ClampMin = "0"))
	int32 MaxNumRandomBranches;

	// On this platform, use the specified quality at this index to override the quality used for SoundCues on this platform
	UPROPERTY(EditAnywhere, Category = "SoundCueLoading", meta = (DisplayName = "Quality Index for Sound Cues", ClampMin = "0", ClampMax = "50"))
	int32 SoundCueQualityIndex = 1;

	AUDIOPLATFORMCONFIGURATION_API FPlatformRuntimeAudioCompressionOverrides();

	// Get singleton containing default settings for compression.
	static FPlatformRuntimeAudioCompressionOverrides* GetDefaultCompressionOverrides()
	{
		if (DefaultCompressionOverrides == nullptr)
		{
			DefaultCompressionOverrides = new FPlatformRuntimeAudioCompressionOverrides();
		}

		return DefaultCompressionOverrides;
	}

private:
	static AUDIOPLATFORMCONFIGURATION_API FPlatformRuntimeAudioCompressionOverrides* DefaultCompressionOverrides;
};
