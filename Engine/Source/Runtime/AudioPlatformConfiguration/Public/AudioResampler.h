// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Logging/LogMacros.h"
#include "Templates/UniquePtr.h"

// TODO: Move BufferVectorOperations AUDIO_BUFFER_ALIGNMENT
// define to more central location and reference here.
#define AUDIO_BUFFER_ALIGNMENT 16

DECLARE_LOG_CATEGORY_EXTERN(LogAudioResampler, Warning, All);

namespace Audio
{
	// TODO: Move BufferVectorOperations Aligned...Buffer
	// defines to more central location and reference here.
	namespace VectorOps
	{
		using FAlignedFloatBuffer = TArray<float, TAlignedHeapAllocator<AUDIO_BUFFER_ALIGNMENT>>;
	} // namespace VectorOps

	enum class EResamplingMethod : uint8
	{
		BestSinc = 0,
		ModerateSinc = 1,
		FastSinc = 2,
		ZeroOrderHold = 3,
		Linear = 4
	};

	struct FResamplingParameters
	{
		EResamplingMethod ResamplerMethod;
		int32 NumChannels;
		float SourceSampleRate;
		float DestinationSampleRate;
		VectorOps::FAlignedFloatBuffer& InputBuffer;
	};

	struct FResamplerResults
	{
		VectorOps::FAlignedFloatBuffer* OutBuffer;

		float ResultingSampleRate;

		int32 InputFramesUsed;

		int32 OutputFramesGenerated;

		FResamplerResults()
			: OutBuffer(nullptr)
			, ResultingSampleRate(0.0f)
			, InputFramesUsed(0)
			, OutputFramesGenerated(0)
		{}
	};

	// Get how large the output buffer should be for a resampling operation.
	AUDIOPLATFORMCONFIGURATION_API int32 GetOutputBufferSize(const FResamplingParameters& InParameters);

	// Simple, inline resampler. Returns true on success, false otherwise.
	AUDIOPLATFORMCONFIGURATION_API bool Resample(const FResamplingParameters& InParameters, FResamplerResults& OutData);


	class FResamplerImpl;

	class FResampler
	{
	public:
		AUDIOPLATFORMCONFIGURATION_API FResampler();
		AUDIOPLATFORMCONFIGURATION_API ~FResampler();

		AUDIOPLATFORMCONFIGURATION_API void Init(EResamplingMethod ResamplingMethod, float StartingSampleRateRatio, int32 InNumChannels);
		AUDIOPLATFORMCONFIGURATION_API void SetSampleRateRatio(float InRatio);
		AUDIOPLATFORMCONFIGURATION_API int32 ProcessAudio(float* InAudioBuffer, int32 InSamples, bool bEndOfInput, float* OutAudioBuffer, int32 MaxOutputFrames, int32& OutNumFrames);

	private:
		TUniquePtr<FResamplerImpl> CreateImpl();
		TUniquePtr<FResamplerImpl> Impl;
	};
	
} // namespace Audio
